{
	"FileVersion" : 1,
	"Version" : 1,
	"VersionName" : "0.1",
	"FriendlyName" : "Movie Scene Pose Search Tracks",
	"Description" : "Sequencer pose search tracks using the Anim Mixer",
	"Category" : "Animation",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed" : false,
	"Plugins":
	[
		{
			"Name": "MovieSceneAnimMixer",
			"Enabled": true
		},
		{
			"Name": "UAFPoseSearch",
			"Enabled": true
		}
	],
	"Modules" :
	[
		{
			"Name" : "MovieScenePoseSearchTracks",
			"Type" : "Runtime",
			"LoadingPhase" : "Default"
		},
		{
			"Name" : "MovieScenePoseSearchTracksEditor",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}
