// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScenePoseSearchTracksModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "MovieSceneAnimMixerModule.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "IMovieSceneModule.h"
#include "Sections/MovieSceneStitchAnimSection.h"
#include "PoseSearch/PoseHistoryProvider.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "Animation/AnimNodeBase.h"
#include "EvaluationVM/EvaluationVM.h"

namespace UE::MovieScene
{

void FMovieScenePoseSearchTracksModule::StartupModule()
{
	IMovieSceneModule::Get().RegisterCompatibleDecoration(UMovieSceneStitchAnimSection::StaticClass(), UMovieSceneRootMotionSettingsDecoration::StaticClass());
}

void FMovieScenePoseSearchTracksModule::ShutdownModule()
{
	if (IMovieSceneModule::IsAvailable() && UObjectInitialized())
	{
		IMovieSceneModule::Get().UnregisterCompatibleDecoration(UMovieSceneStitchAnimSection::StaticClass(), UMovieSceneRootMotionSettingsDecoration::StaticClass());
	}
}

}

IMPLEMENT_MODULE(UE::MovieScene::FMovieScenePoseSearchTracksModule, MovieScenePoseSearchTracks)
