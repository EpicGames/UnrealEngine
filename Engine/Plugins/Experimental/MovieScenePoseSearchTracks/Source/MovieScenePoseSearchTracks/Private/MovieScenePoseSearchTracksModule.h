// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

struct FAnimationUpdateContext;

namespace UE::UAF
{
	struct FEvaluationVM;
}

namespace UE::MovieScene
{

class FMovieScenePoseSearchTracksModule : public IModuleInterface
{
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

}
