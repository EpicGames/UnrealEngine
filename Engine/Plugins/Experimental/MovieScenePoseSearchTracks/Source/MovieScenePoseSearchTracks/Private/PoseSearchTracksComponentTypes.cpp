// Copyright Epic Games, Inc. All Rights Reserved.

#include "PoseSearchTracksComponentTypes.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

namespace UE::MovieScene
{
	static TUniquePtr<FPoseSearchTracksComponentTypes> GPoseSearchTracksComponentTypes;
	static bool GPoseSearchTracksComponentTypesDestroyed = false;

	FPoseSearchTracksComponentTypes* FPoseSearchTracksComponentTypes::Get()
	{
		if (!GPoseSearchTracksComponentTypes.IsValid())
		{
			check(!GPoseSearchTracksComponentTypesDestroyed);
			GPoseSearchTracksComponentTypes.Reset(new FPoseSearchTracksComponentTypes);
		}
		return GPoseSearchTracksComponentTypes.Get();
	}

	void FPoseSearchTracksComponentTypes::Destroy()
	{
		GPoseSearchTracksComponentTypes.Reset();
		GPoseSearchTracksComponentTypesDestroyed = true;
	}

	FPoseSearchTracksComponentTypes::FPoseSearchTracksComponentTypes()
	{
		FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

		ComponentRegistry->NewComponentType(&StitchAnim, TEXT("Stitch Animation"));

		ComponentRegistry->Factories.DuplicateChildComponent(StitchAnim);

	}

}