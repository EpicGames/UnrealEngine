// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneStitchAnimSection.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimPoseSearchProvider.h"
#include "Animation/AnimSequence.h"
#include "Animation/BuiltInAttributeTypes.h"
#include "AnimMixerComponentTypes.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EvaluationVM/Tasks/DecompressionTools.h"
#include "MovieScene.h"
#include "MovieSceneSequence.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "MovieSceneTracksComponentTypes.h"
#include "PoseSearch/PoseSearchContext.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "PoseSearch/PoseSearchFeatureChannel_PermutationTime.h"
#include "PoseSearch/PoseSearchHistory.h"
#include "PoseSearch/PoseSearchLibrary.h"
#include "PoseSearch/PoseSearchSchema.h"
#include "PoseSearchTracksComponentTypes.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"
#include "Tracks/MovieSceneStitchAnimTrack.h"
#include "Variants/MovieSceneTimeWarpVariantPayloads.h"
#include "VisualLogger/VisualLogger.h"
#include "PoseHistoryEvaluation.h"
#include "PoseSearch/AnimNode_PoseSearchHistoryCollector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneStitchAnimSection)

#define LOCTEXT_NAMESPACE "MovieSceneStitchAnimSection"

namespace UE::MovieScene
{
	// @todo: we could inherit from MemStackPoseHistory and just calcualte future poses here rather than relying on FPoseSearchFutureProperties
	// Experimental, this feature might be removed without warning, not for production use
	struct FOverridePoseHistory : UE::PoseSearch::FMemStackPoseHistory
	{
		virtual const FTransformTrajectory& GetTrajectory() const override
		{
			return Trajectory;
		}

		virtual void SetTrajectory(const FTransformTrajectory& InTrajectory, float InTrajectorySpeedMultiplier = 1.f) override
		{
			Trajectory = InTrajectory;
		}

		FTransformTrajectory Trajectory;
	};
} // namespace UE::MovieScene

void FMovieSceneStitchAnimEvaluationTask::Execute(UE::UAF::FEvaluationVM& VM) const
{
	using namespace UE::UAF;
	using namespace UE::MovieScene;
	using namespace UE::PoseSearch;
	using namespace UE::Anim;

	if (VM.GetActiveNamedSet())
	{
		// TODO: Implement with new attribute runtime
		FValueBundleStack Collection(VM.GetActiveNamedSet());
		Collection.InitWithValueSpace(FValueSpace(EValueSpaceType::Local));

		VM.PushValue(ATTRIBUTE_STACK_NAME, MakeUnique<FValueBundle>(MoveTemp(Collection)));
		return;
	}

	const FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	// We rely on the pose history having been stored first.
	// TODO: Find a way to initialize a pose history here for the first time and store it in something the task can access (stitch anim system?)
	// if not already present.
	const IPoseHistory* PoseHistory = nullptr;
	if (const TUniquePtr<FPoseHistoryEvaluationHelper>* PoseHistoryEvalHelper = VM.PeekValue<TUniquePtr<FPoseHistoryEvaluationHelper>>(POSEHISTORY_STACK_NAME, 0))
	{
		PoseHistory = (*PoseHistoryEvalHelper)->PoseHistoryPtr.Get();
	}

	const UObject* AnimContext = ContextObject.Get();
	const UAnimInstance* AnimInstance = Cast<UAnimInstance>(AnimContext);
	if (PoseHistory == nullptr && AnimInstance != nullptr && !StitchData.OverridePoseHistoryTag.IsNone())
	{
		if (const FAnimNode_PoseSearchHistoryCollector_Base* PoseHistoryNode = UPoseSearchLibrary::FindPoseHistoryNode(StitchData.OverridePoseHistoryTag, AnimInstance))
		{
			PoseHistory = PoseHistoryNode->GetPoseHistoryPtr();
		}
	}

	// If we haven't yet run a search and found an animation, do that
	if (!MatchedAsset)
	{
		if (PoseHistory && StitchData.StitchDatabase && StitchData.TargetPoseAsset)
		{
			// TODO: Potentially I can use this if I swap to a model of repeatedly searching each frame
				/*IPoseSearchProvider::FSearchPlayingAsset PlayingAsset;
				PlayingAsset.Asset = GetAnimAsset();
				PlayingAsset.AccumulatedTime = GetAccumulatedTime();*/

			// set up the override pose history we're using for the UPoseSearchLibrary::MotionMatch. We initiale it with PoseHistory as fallback for non overridden methods
			FOverridePoseHistory OverridePoseHistory;
			OverridePoseHistory.Init(PoseHistory);

			// computing the overridden trajectory with all the past FTransformTrajectorySample from the PoseHistory we found via Pose History Stack, 
			// and one prediction future sample computed from StitchData.TargetTransform and TimeToTarget
			const TArray<FTransformTrajectorySample>& TrajectorySamples = PoseHistory->GetTrajectory().Samples;
			OverridePoseHistory.Trajectory.Samples.Reserve(TrajectorySamples.Num()); // over estimating memory requirements to avoid reallocations
			for (const FTransformTrajectorySample& TrajectorySample : TrajectorySamples)
			{
				// we want to only collect the past trajectory, not the prediction for now
				// @todo: calculate the full trajectory in Sequencer!
				if (TrajectorySample.TimeInSeconds > 0.f)
				{
					break;
				}

				OverridePoseHistory.Trajectory.Samples.Add(TrajectorySample);
			}
			// @todo: FutureTrajectorySample should contain the position / facing or the mesh in world space,
			//        not the world space transform StitchData.TargetTransform that seems to be representing the actor transform!
			FTransformTrajectorySample FutureTrajectorySample;

			const FTransform MeshTargetTransform = MeshToActorTransform * StitchData.TargetTransform;
			FutureTrajectorySample.Position = MeshTargetTransform.GetLocation();
			FutureTrajectorySample.Facing = MeshTargetTransform.GetRotation();
			FutureTrajectorySample.TimeInSeconds = TimeToTarget;
			OverridePoseHistory.Trajectory.Samples.Add(FutureTrajectorySample);

			// extracting two future poses from StitchData.TargetPoseAsset at StitchData.TargetAnimationTimeSeconds in the future and StitchData.TargetAnimationTimeSeconds - FiniteDelta
			// to have motion matching been able to compute velocities (if required by the database schema channels)
			OverridePoseHistory.ExtractAndAddFuturePoses(StitchData.TargetPoseAsset, StitchData.TargetAnimationTimeSeconds, FiniteDelta, FVector::ZeroVector, TimeToTarget, nullptr, true);

			PoseHistory = &OverridePoseHistory;

			FPoseSearchContinuingProperties ContinuingProperties;
			/*ContinuingProperties.PlayingAsset = PlayingAsset.Asset;
			ContinuingProperties.PlayingAssetAccumulatedTime = PlayingAsset.AccumulatedTime;*/

			const UObject* StitchDatabaseObj = StitchData.StitchDatabase.Get();
			FSearchResults_Single SearchResults;
			UPoseSearchLibrary::MotionMatch(MakeArrayView(&AnimContext, 1), MakeArrayView(&DefaultRole, 1), MakeArrayView(&PoseHistory, 1),
				MakeArrayView(&StitchDatabaseObj, 1), ContinuingProperties, TimeToTarget, FPoseSearchEvent(), SearchResults);
				
			const FSearchResult SearchResult = SearchResults.GetBestResult();
			if (const FSearchIndexAsset* SearchIndexAsset = SearchResult.GetSearchIndexAsset())
			{
				const UPoseSearchDatabase* Database = SearchResult.Database.Get();
				if (const FPoseSearchDatabaseAnimationAssetBase* DatabaseAnimationAssetBase = Database->GetDatabaseAnimationAsset(SearchIndexAsset->GetSourceAssetIdx()))
				{
					MatchedAsset = Cast<UAnimationAsset>(DatabaseAnimationAssetBase->GetAnimationAsset());
					MatchedAssetTime = SearchResult.GetAssetTime();

					// TODO: Add mirroring support
					//ProviderResult.bMirrored = SearchIndexAsset->IsMirrored();

					// figuring out the actual interval time (that will impact the animation play rate, since we'll be extracting poses using the parametric time along the duration of the stitch)
					MatchedAssetActualIntervalTime = TimeToTarget;
					if (StitchData.TargetPoseAsset)
					{
						if (const UPoseSearchFeatureChannel_PermutationTime* PermutationTimeChannel = Database->Schema->FindFirstChannelOfType<UPoseSearchFeatureChannel_PermutationTime>())
						{
							const FSearchIndex& SearchIndex = Database->GetSearchIndex();
							if (!SearchIndex.IsValuesEmpty())
							{
								TConstArrayView<float> ResultData = Database->GetSearchIndex().GetPoseValues(SearchResult.PoseIdx);
								MatchedAssetActualIntervalTime = PermutationTimeChannel->GetPermutationTime(ResultData);
							}
						}
					}
				}
			}
		}
	}

	if (!MatchedAsset)
	{
		constexpr bool bIsAdditive = false;
		FKeyframeState ReferenceKeyframe = VM.MakeReferenceKeyframe(bIsAdditive);
		VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(ReferenceKeyframe)));
		return;
	}

	FKeyframeState Keyframe = VM.MakeUninitializedKeyframe(false);
	// TODO: For now assume this asset is a UAnimSequence

	UAnimSequence* AnimSequence = Cast<UAnimSequence>(MatchedAsset);
	if (!AnimSequence)
	{
		constexpr bool bIsAdditive = false;
		FKeyframeState ReferenceKeyframe = VM.MakeReferenceKeyframe(bIsAdditive);
		VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(ReferenceKeyframe)));
		return;
	}

	// Calculate the root transform by warping linearly the animation root transforms to the target StitchData.TargetTransform (after converting it to the correct animation space in MeshComponent space)
	const float TotalTime = TimeToTarget + CurrentTime - InitialTime;
	float CurrentBlendWeight = 1.f;
	float PreviousBlendWeight = 1.f;
	if (TotalTime > UE_SMALL_NUMBER)
	{
		CurrentBlendWeight = (CurrentTime - InitialTime) / TotalTime;
		PreviousBlendWeight = (PreviousTime - InitialTime) / TotalTime;
	}

	const float AnimationTime =  MatchedAssetTime + CurrentBlendWeight * MatchedAssetActualIntervalTime;
	const float PreviousAnimationTime = MatchedAssetTime + PreviousBlendWeight * MatchedAssetActualIntervalTime;
	const float AnimationTargetTime = MatchedAssetTime + MatchedAssetActualIntervalTime;

	// mesh space animation transforms associate to the beginning of the stitch track AnimSpaceInitial, end AnimSpaceTarget, and current AnimSpaceCurrent
	const FTransform AnimSpaceInitial = AnimSequence->ExtractRootTrackTransform(FAnimExtractContext((double)MatchedAssetTime), nullptr);
	const FTransform AnimSpaceCurrent = AnimSequence->ExtractRootTrackTransform(FAnimExtractContext((double)AnimationTime), nullptr);
	const FTransform AnimSpaceTarget = AnimSequence->ExtractRootTrackTransform(FAnimExtractContext((double)AnimationTargetTime), nullptr);

	// local delta transforms from the beginning of the animation to the end (InitialToTarget), and current playback time (InitialToCurrent)
	const FTransform InitialToTarget = AnimSpaceTarget.GetRelativeTransform(AnimSpaceInitial);
	const FTransform InitialToCurrent = AnimSpaceCurrent.GetRelativeTransform(AnimSpaceInitial);

	// calculating transforms in mesh space for the beginning of the stitch..
	const FTransform InitialMeshTransform = MeshToActorTransform * InitialRootTransform;
	// ..final transform where we want the actor to be placed..
	const FTransform FinalMeshTransform = MeshToActorTransform * StitchData.TargetTransform;
	// ..where the animation will bring the actor in case no warping is applied..
	const FTransform FinalAnimationTransform = InitialToTarget * InitialMeshTransform;
	// ..where the animation would place the actor in case no warping is applied..
	const FTransform CurrentAnimationTransform = InitialToCurrent * InitialMeshTransform;

	// calculating the warping alignement error, as required delta transform from where the animation would end up, and where we want to place the actor
	const FTransform AlignmentError = FinalMeshTransform.GetRelativeTransform(FinalAnimationTransform);
	// blending the AlignmentError by BlendWeight
	FTransform BlendedAlignmentError;
	BlendedAlignmentError.Blend(FTransform::Identity, AlignmentError, CurrentBlendWeight);
	// calculating where the mesh want to be with after applying the animation AND the blended portion from the alignement error (BlendedAlignmentError)
	const FTransform CurrentMeshTransform = BlendedAlignmentError * CurrentAnimationTransform;
	// CurrentMeshTransform converted back to world actor space, used only for VLOG debugging below.
	const FTransform CurrentRootTransform = MeshToActorTransform.GetRelativeTransformReverse(CurrentMeshTransform);

	FDeltaTimeRecord DeltaTime;
	DeltaTime.Set(PreviousAnimationTime, AnimationTime-PreviousAnimationTime);
	const FAnimExtractContext ExtractionContext((double)AnimationTime, false, DeltaTime, false);
	const bool bUseRawData = FDecompressionTools::ShouldUseRawData(AnimSequence, ExtractionContext);

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Bones))
	{
		FDecompressionTools::GetAnimationPose(AnimSequence, ExtractionContext, Keyframe.Pose, bUseRawData);

		// Stash the warped world-space transform on the root bone so the standard
		// FAnimNextExtractRootMotionTask will lift it onto RootTransformAttribute
		// (and zero the bone) the same way it does for every other section.
		const int32 RootIndex = Keyframe.Pose.GetRefPose().GetLODBoneIndexFromSkeletonBoneIndex(0);
		if (RootIndex != INDEX_NONE)
		{
			Keyframe.Pose.LocalTransformsView[RootIndex] = CurrentMeshTransform;
		}
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Curves))
	{
		FDecompressionTools::GetAnimationCurves(AnimSequence, ExtractionContext, Keyframe.Curves, bUseRawData);
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes))
	{
		FDecompressionTools::GetAnimationAttributes(AnimSequence, ExtractionContext, Keyframe.Pose.GetRefPose(), Keyframe.Attributes, bUseRawData);
	}

	if (EnumHasAnyFlags(VM.GetFlags(), EEvaluationFlags::Attributes | EEvaluationFlags::Trajectory))
	{
		if (StitchData.bAuthoritativeRootMotion)
		{
			FIntegerAnimationAttribute* RootMotionIsAuthoritativeAttribute = Keyframe.Attributes.FindOrAdd<FIntegerAnimationAttribute>(AnimMixerComponents->RootTransformIsAuthoritativeAttributeId);
			RootMotionIsAuthoritativeAttribute->Value = 1;
		}
	}

	VM.PushValue(KEYFRAME_STACK_NAME, MakeUnique<FKeyframeState>(MoveTemp(Keyframe)));

#if ENABLE_VISUAL_LOG
	if (FVisualLogger::IsRecording())
	{
		static const TCHAR* LogName = TEXT("MovieSceneStitchAnimSection");

		// actor space debug lines
		UE_VLOG_CIRCLE(ContextObject.Get(), LogName, Display, InitialRootTransform.GetLocation(), FVector::UpVector, 10.f, FColorList::Blue, TEXT(""));
		UE_VLOG_CIRCLE(ContextObject.Get(), LogName, Display, CurrentRootTransform.GetLocation(), FVector::UpVector, 10.f, FColorList::Blue, TEXT(""));
		UE_VLOG_CIRCLE(ContextObject.Get(), LogName, Display, StitchData.TargetTransform.GetLocation(), FVector::UpVector, 10.f, FColorList::Blue, TEXT(""));
		UE_VLOG_SEGMENT(ContextObject.Get(), LogName, Display, InitialRootTransform.GetLocation(), StitchData.TargetTransform.GetLocation(), FColorList::Blue, TEXT(""));
		
		// mesh space animation debug lines
		UE_VLOG_CIRCLE(ContextObject.Get(), LogName, Display, InitialMeshTransform.GetLocation(), FVector::UpVector, 15.f, FColorList::Green, TEXT(""));
		UE_VLOG_CIRCLE(ContextObject.Get(), LogName, Display, CurrentAnimationTransform.GetLocation(), FVector::UpVector, 15.f, FColorList::Green, TEXT(""));
		UE_VLOG_CIRCLE(ContextObject.Get(), LogName, Display, FinalAnimationTransform.GetLocation(), FVector::UpVector, 15.f, FColorList::Green, TEXT(""));
		UE_VLOG_SEGMENT(ContextObject.Get(), LogName, Display, InitialMeshTransform.GetLocation(), FinalAnimationTransform.GetLocation(), FColorList::Green, TEXT(""));
		
		// mesh space full aligned warped animation debug lines
		UE_VLOG_CIRCLE(ContextObject.Get(), LogName, Display, InitialMeshTransform.GetLocation(), FVector::UpVector, 18.f, FColorList::Black, TEXT(""));
		UE_VLOG_CIRCLE(ContextObject.Get(), LogName, Display, CurrentMeshTransform.GetLocation(), FVector::UpVector, 18.f, FColorList::Black, TEXT(""));
		UE_VLOG_CIRCLE(ContextObject.Get(), LogName, Display, FinalMeshTransform.GetLocation(), FVector::UpVector, 18.f, FColorList::Black, TEXT(""));
		UE_VLOG_SEGMENT(ContextObject.Get(), LogName, Display, InitialMeshTransform.GetLocation(), FinalMeshTransform.GetLocation(), FColorList::Black, TEXT(""));
	}
#endif
}

UMovieSceneStitchAnimSection::UMovieSceneStitchAnimSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BlendType = EMovieSceneBlendType::Absolute;

	Weight.SetDefault(1.f);

	UMovieSceneSequence* OuterSequence = GetTypedOuter<UMovieSceneSequence>();

	if (UMovieScene* MovieScene = OuterSequence ? OuterSequence->GetMovieScene() : nullptr)
	{
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();

		FFrameTime EndTime = ConvertFrameTime(FFrameTime(5), DisplayRate, TickResolution);

		SectionRange.Value = TRange<FFrameNumber>(FFrameNumber(0), EndTime.FrameNumber);
	}
	
}

EMovieSceneChannelProxyType UMovieSceneStitchAnimSection::CacheChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR

	static FMovieSceneChannelMetaData MetaData("Weight", LOCTEXT("WeightChannelName", "Weight"));
	MetaData.bCanCollapseToTrack = false;

	Channels.Add(Weight, MetaData, TMovieSceneExternalValue<float>());

#else

	Channels.Add(Weight);

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
	return EMovieSceneChannelProxyType::Dynamic;
}

float UMovieSceneStitchAnimSection::GetTotalWeightValue(FFrameTime InTime) const
{
	float ManualWeight = 1.f;
	Weight.Evaluate(InTime, ManualWeight);
	return ManualWeight * EvaluateEasing(InTime);
}

UObject* UMovieSceneStitchAnimSection::GetSourceObject() const
{
	return StitchDatabase;
}

void UMovieSceneStitchAnimSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& InParams, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	const FPoseSearchTracksComponentTypes* PoseSearchTrackComponents = FPoseSearchTracksComponentTypes::Get();
	const FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	const FGuid ObjectBindingID = InParams.GetObjectBindingID();

	if (!ObjectBindingID.IsValid())
	{
		return;
	}

	FMovieSceneStitchAnimComponentData StitchData;
	StitchData.StitchDatabase = StitchDatabase;
	StitchData.TargetPoseAsset = TargetPoseAsset;
	StitchData.TargetAnimationTimeSeconds = TargetAnimationTimeSeconds;
	StitchData.TargetTransform = TargetTransform;
	StitchData.StartFrame = GetInclusiveStartFrame();
	StitchData.EndFrame = GetExclusiveEndFrame();
	StitchData.TargetTransformSpace = TargetTransformSpace;
	StitchData.bAuthoritativeRootMotion = bAuthoritativeRootMotion;
	StitchData.OverridePoseHistoryTag = OverridePoseHistoryTag;
	
	// Make a default eval task. This will be filled out more by the stitch system
	TSharedPtr<FMovieSceneStitchAnimEvaluationTask> EvalTask = MakeShared<FMovieSceneStitchAnimEvaluationTask>();
	EvalTask->StitchData = StitchData;

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(PoseSearchTrackComponents->StitchAnim, StitchData)
		.Add(BuiltInComponents->GenericObjectBinding, ObjectBindingID)
		.Add(BuiltInComponents->BoundObjectResolver, UMovieSceneSkeletalAnimationSystem::ResolveSkeletalMeshComponentBinding)
		.AddConditional(BuiltInComponents->WeightChannel, &Weight, Weight.HasAnyData())
		.AddTag(TrackComponents->Tags.AnimMixerPoseProducer)
		.Add(AnimMixerComponents->Task, EvalTask)
		.Add(AnimMixerComponents->Target, TInstancedStruct<FMovieSceneMixedAnimationTarget>::Make())
		.Add(AnimMixerComponents->MixerEntry, nullptr)
		.Add(AnimMixerComponents->Priority, 0)
	);
}

double FMovieSceneStitchAnimComponentData::MapTimeToSectionSeconds(FFrameTime InPosition, FFrameRate InFrameRate) const
{
	return InFrameRate.AsSeconds(InPosition - StartFrame);
}

#undef LOCTEXT_NAMESPACE

