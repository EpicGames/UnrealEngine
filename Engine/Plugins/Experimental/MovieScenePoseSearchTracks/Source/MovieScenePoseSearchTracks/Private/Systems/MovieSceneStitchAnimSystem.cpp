// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneStitchAnimSystem.h"

#include "Async/TaskGraphInterfaces.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "AnimMixerComponentTypes.h"
#include "PoseSearchTracksComponentTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneBoundSceneComponentInstantiator.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneStitchAnimSection.h"
#include "EvaluationVM/EvaluationVM.h"
#include "EvaluationVM/KeyframeState.h"
#include "Tracks/IMovieSceneTransformOrigin.h"
#include "Systems/MovieSceneTransformOriginSystem.h"
#include "Components/SkeletalMeshComponent.h"
#include "IMovieScenePlaybackClient.h"
#include "Animation/AnimInstance.h"
#include "Component/AnimNextComponent.h"
#include "Systems/MovieSceneAnimNextTargetSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneStitchAnimSystem)


namespace UE::MovieScene
{
	/** ------------------------------------------------------------------------- */
	/** Updates the stitch anim evaluation task each frame*/
	struct FUpdateStitchAnimTask
	{
		const UMovieSceneEntitySystemLinker* EntityLinker;
		const UMovieSceneTransformOriginSystem* TransformOriginSystem;

		FUpdateStitchAnimTask(const UMovieSceneEntitySystemLinker* InLinker)
			: EntityLinker(InLinker)
		{
			TransformOriginSystem = EntityLinker->FindSystem<UMovieSceneTransformOriginSystem>();
		}

		// Chooses a context object for motion matching calls based on the anim target for the track
		// TODO: Ideally this would be more encapsulated in the target itself, or the motion matching call wouldn't require this
		const UObject* GetContextObject(TInstancedStruct<FMovieSceneMixedAnimationTarget> Target, const UObject* BoundObject) const
		{
			const UObject* ContextObject = nullptr;

			// If we're targeting anim next, try to grab the anim next component
			auto TryFindComponent = []<typename T>(const UObject* BoundObject) -> const T*
				{
					if (const AActor* Actor = Cast<AActor>(BoundObject))
					{
						return Actor->FindComponentByClass<T>();
					}
					else if (const UActorComponent* ActorComponent = Cast<UActorComponent>(BoundObject))
					{
						if (ActorComponent->GetClass() == T::StaticClass())
						{
							return Cast<T>(ActorComponent);
						}
						else 
						{
							return ActorComponent->GetOwner() ? ActorComponent->GetOwner()->FindComponentByClass<T>() : nullptr;
						}
					}
					return nullptr;
				};

			if (Target.IsValid() && Target.GetScriptStruct() == FMovieSceneAnimNextInjectionTarget::StaticStruct())
			{
				ContextObject = TryFindComponent.template operator()<UUAFComponent>(BoundObject);
			}
			else if (!Target.IsValid() || Target.GetScriptStruct() == FMovieSceneMixedAnimationTarget::StaticStruct())
			{
				// If we specifically didn't choose a target, try seeing if we should use anim next
				const UUAFComponent* AnimNextComponent = TryFindComponent.template operator()<UUAFComponent>(BoundObject);
				const USkeletalMeshComponent* SkeletalMeshComponent = TryFindComponent.template operator()<USkeletalMeshComponent>(BoundObject);
				if (AnimNextComponent && SkeletalMeshComponent && !SkeletalMeshComponent->bEnableAnimation)
				{
					ContextObject = AnimNextComponent;
				}
			}

			// Fall back to using the skeletal mesh component
			if (!ContextObject)
			{
				if (const USkeletalMeshComponent* MeshComponent = TryFindComponent.template operator()<USkeletalMeshComponent>(BoundObject))
				{
					ContextObject = MeshComponent->GetAnimInstance();
				}
			}

			return ContextObject;
		}

		void ForEachAllocation(
			const FEntityAllocationProxy AllocationProxy,
			TRead<FInstanceHandle> InstanceHandles,
			TRead<FMovieSceneStitchAnimComponentData> StitchAnims,
			TRead<UObject*> BoundObjects,
			TRead<TInstancedStruct<FMovieSceneMixedAnimationTarget>> Targets,
			TReadOptional<FFrameTime> OptionalEvalTimes,
			TWrite<TSharedPtr<FAnimNextEvaluationTask>> EvalTask) const
		{
			const FEntityAllocation* Allocation = AllocationProxy.GetAllocation();
			const int32 Num = Allocation->Num();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FInstanceHandle& InstanceHandle(InstanceHandles[Index]);
				const FMovieSceneStitchAnimComponentData& StitchAnim(StitchAnims[Index]);

				// Get the full context, so we can get both the current and previous evaluation times.
				const FSequenceInstance& SequenceInstance = EntityLinker->GetInstanceRegistry()->GetInstance(InstanceHandle);
				const FMovieSceneContext& Context = SequenceInstance.GetContext();
				const FFrameTime EvalFrameTime = OptionalEvalTimes ? OptionalEvalTimes[Index] : Context.GetTime();
				const UObject* BoundObject = BoundObjects[Index];

				float PreviousTimeSeconds = StitchAnim.MapTimeToSectionSeconds(Context.GetPreviousTime(), Context.GetFrameRate());
				float CurrentTimeSeconds = StitchAnim.MapTimeToSectionSeconds(EvalFrameTime, Context.GetFrameRate());

				TSharedPtr<FMovieSceneStitchAnimEvaluationTask> AnimTask = StaticCastSharedPtr<FMovieSceneStitchAnimEvaluationTask>(EvalTask[Index]);

				if (!BoundObject || !StitchAnim.StitchDatabase || !StitchAnim.TargetPoseAsset || !AnimTask.IsValid())
				{
					continue;
				}

				AnimTask->TimeToTarget = StitchAnim.MapTimeToSectionSeconds(StitchAnim.EndFrame, Context.GetFrameRate()) - CurrentTimeSeconds;
				
				if (AnimTask->InitialTime < 0.0f)
				{
					// We use the previous time for the initial time, as it will match the transform from the actor which hasn't
					// yet been set for this frame.
					AnimTask->InitialTime = PreviousTimeSeconds;
					if (StitchAnim.TargetTransformSpace == EMovieSceneRootMotionSpace::AnimationSpace && TransformOriginSystem)
					{
						TransformOriginSystem->GetTransformOrigin(InstanceHandle, AnimTask->SequenceTransformOrigin);
					}

					const AActor* Actor = Cast<AActor>(BoundObject);
					if (!Actor)
					{
						if (const UActorComponent* ActorComponent = Cast<UActorComponent>(BoundObject))
						{
							Actor = ActorComponent->GetOwner();
						}
					}
					
					const USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject);
					if (!SkeletalMeshComponent && Actor)
					{
						SkeletalMeshComponent = Actor->GetComponentByClass<USkeletalMeshComponent>();
					}

					AnimTask->ContextObject = GetContextObject(Targets[Index], BoundObject);

					if (Actor)
					{
						AnimTask->InitialRootTransform = Actor->GetRootComponent()->GetRelativeTransform();

						if (SkeletalMeshComponent)
						{
							AnimTask->MeshToActorTransform = SkeletalMeshComponent->GetRelativeTransform();
						}
					}
				}
				AnimTask->PreviousTime = PreviousTimeSeconds;
				AnimTask->CurrentTime = CurrentTimeSeconds;
			}
		}
	};

} // namespace UE::MovieScene

UMovieSceneStitchAnimSystem::UMovieSceneStitchAnimSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FPoseSearchTracksComponentTypes* PoseSearchTrackComponents = FPoseSearchTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();
	RelevantComponent = PoseSearchTrackComponents->StitchAnim;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneGenericBoundObjectInstantiator::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBoundSceneComponentInstantiator::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneTransformOriginSystem::StaticClass(), GetClass());
		DefineComponentConsumer(GetClass(), PoseSearchTrackComponents->StitchAnim);
		DefineImplicitPrerequisite(GetClass(), UMovieSceneRestorePreAnimatedStateSystem::StaticClass());
		DefineComponentProducer(GetClass(), AnimMixerComponents->Task);

	}
	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();
}

void UMovieSceneStitchAnimSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FPoseSearchTracksComponentTypes* PoseSearchTrackComponents = FPoseSearchTracksComponentTypes::Get();
	FAnimMixerComponentTypes* AnimMixerComponents = FAnimMixerComponentTypes::Get();

	FTaskID UpdateTask = FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(PoseSearchTrackComponents->StitchAnim)
		.Read(BuiltInComponents->BoundObject)
		.Read(AnimMixerComponents->Target)
		.ReadOptional(BuiltInComponents->EvalTime)
		.Write(AnimMixerComponents->Task)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Schedule_PerAllocation<FUpdateStitchAnimTask>(&Linker->EntityManager, TaskScheduler,
			Linker);
}
