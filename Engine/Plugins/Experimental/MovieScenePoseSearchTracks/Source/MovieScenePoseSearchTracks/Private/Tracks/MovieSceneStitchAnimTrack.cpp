// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneStitchAnimTrack.h"
#include "Sections/MovieSceneStitchAnimSection.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "MovieScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneStitchAnimTrack)

#define LOCTEXT_NAMESPACE "MovieSceneStitchAnimTrack"

UMovieSceneStitchAnimTrack::UMovieSceneStitchAnimTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(124, 15, 124, 65);
	bSupportsDefaultSections = false;
#endif

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);

	EvalOptions.bCanEvaluateNearestSection = true;
}

const TArray<UMovieSceneSection*>& UMovieSceneStitchAnimTrack::GetAllSections() const
{
	return AnimationSections;
}


bool UMovieSceneStitchAnimTrack::SupportsMultipleRows() const
{
	return true;
}

bool UMovieSceneStitchAnimTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneStitchAnimTrack::StaticClass();
}

UMovieSceneSection* UMovieSceneStitchAnimTrack::CreateNewSection()
{
	return NewObject<UMovieSceneStitchAnimSection>(this, NAME_None, RF_Transactional);
}

bool UMovieSceneStitchAnimTrack::HasSection(const UMovieSceneSection& Section) const
{
	return AnimationSections.Contains(&Section);
}

void UMovieSceneStitchAnimTrack::AddSection(UMovieSceneSection& Section)
{
	AnimationSections.Add(&Section);
}

void UMovieSceneStitchAnimTrack::RemoveSection(UMovieSceneSection& Section)
{
	AnimationSections.Remove(&Section);
}

void UMovieSceneStitchAnimTrack::RemoveSectionAt(int32 SectionIndex)
{
	AnimationSections.RemoveAt(SectionIndex);
}

bool UMovieSceneStitchAnimTrack::IsEmpty() const
{
	return AnimationSections.Num() == 0;
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneStitchAnimTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Stitch");
}

#endif

UMovieSceneSection* UMovieSceneStitchAnimTrack::AddNewAnimationOnRow(FFrameNumber KeyTime, UPoseSearchDatabase* PoseSearchDatabase, int32 RowIndex)
{
	UMovieSceneStitchAnimSection* NewSection = Cast<UMovieSceneStitchAnimSection>(CreateNewSection());
	{
		FFrameTime AnimationLength = 5.0f * GetTypedOuter<UMovieScene>()->GetTickResolution();
		int32 IFrameNumber = AnimationLength.FrameNumber.Value + static_cast<int32>(AnimationLength.GetSubFrame() + 0.5f) + 1;

		NewSection->InitialPlacementOnRow(AnimationSections, KeyTime, IFrameNumber, RowIndex);
		NewSection->StitchDatabase = PoseSearchDatabase;
	}

	AddSection(*NewSection);

	return NewSection;
}

#undef LOCTEXT_NAMESPACE

