// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "Sections/MovieSceneStitchAnimSection.h"

namespace UE::MovieScene
{
	struct FPoseSearchTracksComponentTypes
	{
		MOVIESCENEPOSESEARCHTRACKS_API static FPoseSearchTracksComponentTypes* Get();
		MOVIESCENEPOSESEARCHTRACKS_API static void Destroy();

		TComponentTypeID<FMovieSceneStitchAnimComponentData> StitchAnim;

	private:
		FPoseSearchTracksComponentTypes();
	};

}