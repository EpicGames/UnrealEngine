// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "StructUtils/InstancedStruct.h"
#include "MovieSceneAnimationMixerTrack.h"
#include "Animation/AnimationAsset.h"
#include "EvaluationVM/EvaluationTask.h"
#include "AnimSequencerInstanceProxy.h"
#include "MovieSceneStitchAnimSection.generated.h"

USTRUCT()
struct FMovieSceneStitchAnimComponentData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UObject> StitchDatabase;

	UPROPERTY()
	TObjectPtr<UAnimationAsset> TargetPoseAsset;
	
	UPROPERTY()
	float TargetAnimationTimeSeconds = 0.0f;

	UPROPERTY()
	FTransform TargetTransform;

	UPROPERTY()
	FFrameNumber StartFrame;

	UPROPERTY()
	FFrameNumber EndFrame;

	UPROPERTY()
	EMovieSceneRootMotionSpace TargetTransformSpace = EMovieSceneRootMotionSpace::AnimationSpace;

	UPROPERTY()
	bool bAuthoritativeRootMotion = true;
	
	UPROPERTY()
	FName OverridePoseHistoryTag;

	double MapTimeToSectionSeconds(FFrameTime InPosition, FFrameRate InFrameRate) const;
};

USTRUCT()
struct FMovieSceneStitchAnimEvaluationTask : public FAnimNextEvaluationTask
{
	GENERATED_BODY()

	DECLARE_ANIM_EVALUATION_TASK(FMovieSceneStitchAnimEvaluationTask)

	// Task entry point
	virtual void Execute(UE::UAF::FEvaluationVM& VM) const override;

	UPROPERTY(Transient)
	FMovieSceneStitchAnimComponentData StitchData;

	// Initial sequencer time when the motion matching calculation was done
	UPROPERTY(Transient)
	float InitialTime = -1.f;

	// Time until our target pose
	UPROPERTY(Transient)
	float TimeToTarget = -1.f;

	// Previous sequencer time
	UPROPERTY(Transient)
	float PreviousTime = -1.f;

	// Current sequencer time
	UPROPERTY(Transient)
	float CurrentTime = -1.f;

	UPROPERTY(Transient)
	FTransform InitialRootTransform = FTransform::Identity;

	UPROPERTY(Transient)
	TWeakObjectPtr<const UObject> ContextObject;

	UPROPERTY(Transient)
	FTransform SequenceTransformOrigin = FTransform::Identity;

	// Calculated and cached by the motion match at InitialTime
	UPROPERTY(Transient)
	mutable TObjectPtr<UAnimationAsset> MatchedAsset;

	UPROPERTY(Transient)
	mutable float MatchedAssetTime = 0.f;

	UPROPERTY(Transient)
	FTransform MeshToActorTransform = FTransform::Identity;

	UPROPERTY(Transient)
	mutable float MatchedAssetActualIntervalTime = 0.f;
};


/**
 * Handles generating and playing back transitional skeletal animations from a stitch database.
 */
UCLASS( MinimalAPI, DisplayName="Animation Stitch")
class UMovieSceneStitchAnimSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
	, public IMovieSceneAnimationMixerItemInterface
{
	GENERATED_BODY()

public:

	UMovieSceneStitchAnimSection(const FObjectInitializer& ObjectInitializer);

	UPROPERTY(EditAnywhere, Category = "Stitch", meta=(AllowedClasses = "/Script/PoseSearch.PoseSearchDatabase"))
	TObjectPtr<UObject> StitchDatabase;

	UPROPERTY(EditAnywhere, Category = "Stitch")
	TObjectPtr<UAnimationAsset> TargetPoseAsset;

	// TODO: Is this the best way to specify this?
	UPROPERTY(EditAnywhere, Category = "Stitch")
	float TargetAnimationTimeSeconds = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Stitch")
	FTransform TargetTransform;

	/** The weight curve for this animation section */
	UPROPERTY()
	FMovieSceneFloatChannel Weight;

	UPROPERTY(EditAnywhere, Category = "Stitch")
	EMovieSceneRootMotionSpace TargetTransformSpace = EMovieSceneRootMotionSpace::AnimationSpace;

	UPROPERTY(EditAnywhere, Category = "Stitch")
	bool bAuthoritativeRootMotion = true;

	// When using animation blueprints, can be used to search for a specific pose history node by tag.
	// This allow you to have a pose history node pre-sequencer for blending into sequences,
	// but still capture a separate pose history afterwards for blending out.
	UPROPERTY(EditAnywhere, category = "Stitch")
	FName OverridePoseHistoryTag;

	//~ UMovieSceneSection interface
	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	virtual UObject* GetSourceObject() const override;

	virtual FColor GetMixerItemTint() const override
	{
		return MixerTintOverride;
	}

protected:

	virtual float GetTotalWeightValue(FFrameTime InTime) const override;

	/** ~IMovieSceneEntityProvider interface */
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& InParams, FImportedEntity* OutImportedEntity) override;

	FColor MixerTintOverride = FColor(20, 70, 120, 200);

};
