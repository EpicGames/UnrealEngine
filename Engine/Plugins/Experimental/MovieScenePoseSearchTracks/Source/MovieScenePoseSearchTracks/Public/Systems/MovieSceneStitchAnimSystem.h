// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EvaluationVM/EvaluationTask.h"
#include "MovieSceneStitchAnimSystem.generated.h"

// System to handle updating stitch anim evaluation tasks for the anim mixer.
UCLASS(MinimalAPI)
class UMovieSceneStitchAnimSystem
	: public UMovieSceneEntityInstantiatorSystem
{
public:

	GENERATED_BODY()

	UMovieSceneStitchAnimSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	

};