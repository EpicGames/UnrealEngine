// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneNameableTrack.h"
#include "Sections/MovieSceneStitchAnimSection.h"
#include "MovieSceneStitchAnimTrack.generated.h"


/**
 * Handles generating and playing back transitional skeletal animations from a stitch database.
 */
UCLASS(MinimalAPI)
class UMovieSceneStitchAnimTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:

	UMovieSceneStitchAnimTrack(const FObjectInitializer& ObjectInitializer);

public:

	// UMovieSceneTrack interface

	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool IsEmpty() const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual bool SupportsMultipleRows() const override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

	/** Adds a new animation to this track */
	MOVIESCENEPOSESEARCHTRACKS_API UMovieSceneSection* AddNewAnimationOnRow(FFrameNumber KeyTime, class UPoseSearchDatabase* PoseSearchDatabase, int32 RowIndex);

public:

	/** List of all animation sections */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> AnimationSections;
};








