// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MovieScenePoseSearchTracksEditor : ModuleRules
	{
		public MovieScenePoseSearchTracksEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"MovieScene",
					"MovieSceneTracks",
					"MovieSceneTools",
					"MovieScenePoseSearchTracks",
					"MovieSceneAnimMixer",
					"MovieSceneAnimMixerEditor",
					"AnimGraphRuntime",
					"UAF",
					"UAFAnimGraph",
					"UAFPoseSearch",
					"Sequencer",
					"SequencerCore",
					"SlateCore",
					"Slate",
					"UnrealEd",
					"PoseSearch"
				}
			);
		}
	}
}