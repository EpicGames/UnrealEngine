// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieScenePoseSearchTracksEditorModule.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ISequencerModule.h"
#include "MovieSceneAnimMixerEditor/Private/MovieSceneAnimationMixerTrackEditor.h"
#include "TrackEditors/StitchAnimTrackEditor.h"
#include "Sections/MovieSceneStitchAnimSection.h"

namespace UE::MovieScene
{

void FMovieScenePoseSearchTracksEditorModule::StartupModule()
{
	using namespace UE::Sequencer;

	ISequencerModule& SequencerModule = FModuleManager::Get().LoadModuleChecked<ISequencerModule>("Sequencer");

	// register specialty track editors
	StitchAnimationTrackCreateEditorHandle = SequencerModule.RegisterTrackEditor(FOnCreateTrackEditor::CreateStatic(&FStitchAnimTrackEditor::CreateTrackEditor));
	
	FAnimationMixerTrackEditor::RegisterCustomMixerAnimSection(UMovieSceneStitchAnimSection::StaticClass(), FOnMakeSectionInterfaceDelegate::CreateLambda([](UMovieSceneSection& Section, UMovieSceneTrack& Track, FGuid Guid)
		{
			if (UMovieSceneStitchAnimSection* StitchSection = Cast<UMovieSceneStitchAnimSection>(&Section))
			{
				TSharedPtr<ISequencerSection> SectionPtr = MakeShareable(new FStitchAnimSection(*StitchSection));
				return SectionPtr.ToSharedRef();
			}
			TSharedPtr<ISequencerSection> SectionPtr = MakeShareable(new FSequencerSection(Section));
			return SectionPtr.ToSharedRef();
		}));
	
}

void FMovieScenePoseSearchTracksEditorModule::ShutdownModule()
{
	using namespace UE::Sequencer;

	if (!FModuleManager::Get().IsModuleLoaded("Sequencer"))
	{
		return;
	}

	ISequencerModule& SequencerModule = FModuleManager::Get().GetModuleChecked<ISequencerModule>("Sequencer");

	FAnimationMixerTrackEditor::UnregisterCustomMixerAnimSection(UMovieSceneStitchAnimSection::StaticClass());

	// unregister specialty track editors
	SequencerModule.UnRegisterTrackEditor(StitchAnimationTrackCreateEditorHandle);
}

}

IMPLEMENT_MODULE(UE::MovieScene::FMovieScenePoseSearchTracksEditorModule, MovieScenePoseSearchTracksEditor)
