// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

namespace UE::MovieScene
{

class FMovieScenePoseSearchTracksEditorModule : public IModuleInterface
{
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FDelegateHandle StitchAnimationTrackCreateEditorHandle;
};

}
