// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

// Style set containing icons for the MovieScenePoseSearchTracks editor.
class FMovieScenePoseSearchTracksEditorStyle final : public FSlateStyleSet
{
public:

	FMovieScenePoseSearchTracksEditorStyle()
		: FSlateStyleSet("MovieScenePoseSearchTracksEditorStyle")
	{
		const FVector2D Icon16x16(16.0f, 16.0f);

		SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Experimental/MovieScenePoseSearchTracks/Content/Editor/Slate"));

		Set("Tracks.Stitch", new IMAGE_BRUSH_SVG("Stitch_16", Icon16x16));

		FSlateStyleRegistry::RegisterSlateStyle(*this);
	}

	~FMovieScenePoseSearchTracksEditorStyle()
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*this);
	}

	static FMovieScenePoseSearchTracksEditorStyle& Get()
	{
		static FMovieScenePoseSearchTracksEditorStyle Inst;
		return Inst;
	}
};
