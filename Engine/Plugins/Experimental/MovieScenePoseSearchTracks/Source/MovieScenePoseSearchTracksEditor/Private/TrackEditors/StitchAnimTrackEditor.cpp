// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackEditors/StitchAnimTrackEditor.h"
#include "Sections/MovieSceneStitchAnimSection.h"
#include "Tracks/MovieSceneStitchAnimTrack.h"
#include "GameFramework/Actor.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Engine/SCS_Node.h"
#include "MVVM/ViewModels/ViewDensity.h"
#include "SequencerSectionPainter.h"
#include "MovieScenePoseSearchTracksEditorStyle.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "PoseSearch/PoseSearchDatabase.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "MVVM/Extensions/ITrackExtension.h"
#include "MVVM/ViewModels/ViewModelIterators.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "SequencerSettings.h"
#include "MVVM/Views/ViewUtilities.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "MovieSceneToolHelpers.h"
#include "IMovieSceneAnimMixerItemMenuProvider.h"

namespace StitchAnimEditorConstants
{
	// @todo Sequencer Allow this to be customizable
	const uint32 AnimationTrackHeight = 28;
}

namespace UE::MovieScene
{
	// Stitch sections always produce world-space root motion via motion matching.
	// Attach a Root Motion Settings decoration defaulting to WorldSpace so the
	// mixer's space-conversion pipeline applies the correct compensation.
	static void AddDefaultStitchRootMotionDecoration(UMovieSceneStitchAnimSection* Section)
	{
		if (!Section)
		{
			return;
		}

		UMovieSceneRootMotionSettingsDecoration* RootMotionSettings = NewObject<UMovieSceneRootMotionSettingsDecoration>(
			Section, UMovieSceneRootMotionSettingsDecoration::StaticClass(), NAME_None, RF_Transactional);
		RootMotionSettings->RootMotionSpace.SetDefault((uint8)EMovieSceneRootMotionSpace::WorldSpace);
		Section->AddDecoration(RootMotionSettings);
	}
}

#define LOCTEXT_NAMESPACE "FStitchAnimTrackEditor"

USkeletalMeshComponent* AcquireSkeletalMeshFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr)
{
	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;

	if (AActor* Actor = Cast<AActor>(BoundObject))
	{
		if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Actor->GetRootComponent()))
		{
			return SkeletalMeshComponent;
		}

		TArray<USkeletalMeshComponent*> SkeletalMeshComponents;
		Actor->GetComponents(SkeletalMeshComponents);

		if (SkeletalMeshComponents.Num() == 1)
		{
			return SkeletalMeshComponents[0];
		}
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject))
	{
		if (SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			return SkeletalMeshComponent;
		}
	}

	return nullptr;
}

USkeleton* GetSkeletonFromComponent(UActorComponent* InComponent)
{
	USkeletalMeshComponent* SkeletalMeshComp = Cast<USkeletalMeshComponent>(InComponent);
	if (SkeletalMeshComp && SkeletalMeshComp->GetSkeletalMeshAsset() && SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton())
	{
		// @todo Multiple actors, multiple components
		return SkeletalMeshComp->GetSkeletalMeshAsset()->GetSkeleton();
	}

	return nullptr;
}

// Get the skeletal mesh components from the guid
// If bGetSingleRootComponent - return only the root component if it is a skeletal mesh component. 
// This allows the root object binding to have an animation track without needing a skeletal mesh component binding
//
TArray<USkeletalMeshComponent*> AcquireSkeletalMeshComponentsFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr, const bool bGetSingleRootComponent = true)
{
	TArray<USkeletalMeshComponent*> SkeletalMeshComponents;

	UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(Guid) : nullptr;

	AActor* Actor = Cast<AActor>(BoundObject);

	if (!Actor)
	{
		if (UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(BoundObject))
		{
			Actor = ChildActorComponent->GetChildActor();
		}
	}

	if (Actor)
	{
		if (bGetSingleRootComponent)
		{
			if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Actor->GetRootComponent()))
			{
				SkeletalMeshComponents.Add(SkeletalMeshComponent);
				return SkeletalMeshComponents;
			}
		}

		Actor->GetComponents(SkeletalMeshComponents);
		if (SkeletalMeshComponents.Num())
		{
			return SkeletalMeshComponents;
		}

		AActor* ActorCDO = Cast<AActor>(Actor->GetClass()->GetDefaultObject());
		if (ActorCDO)
		{
			if (bGetSingleRootComponent)
			{
				if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ActorCDO->GetRootComponent()))
				{
					SkeletalMeshComponents.Add(SkeletalMeshComponent);
					return SkeletalMeshComponents;
				}
			}

			ActorCDO->GetComponents(SkeletalMeshComponents);
			if (SkeletalMeshComponents.Num())
			{
				return SkeletalMeshComponents;
			}
		}

		UBlueprintGeneratedClass* ActorBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(Actor->GetClass());
		if (ActorBlueprintGeneratedClass && ActorBlueprintGeneratedClass->SimpleConstructionScript)
		{
			const TArray<USCS_Node*>& ActorBlueprintNodes = ActorBlueprintGeneratedClass->SimpleConstructionScript->GetAllNodes();

			for (USCS_Node* Node : ActorBlueprintNodes)
			{
				if (Node->ComponentClass->IsChildOf(USkeletalMeshComponent::StaticClass()))
				{
					if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(Node->GetActualComponentTemplate(ActorBlueprintGeneratedClass)))
					{
						SkeletalMeshComponents.Add(SkeletalMeshComponent);
					}
				}
			}

			if (SkeletalMeshComponents.Num())
			{
				return SkeletalMeshComponents;
			}
		}
	}
	else if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(BoundObject))
	{
		SkeletalMeshComponents.Add(SkeletalMeshComponent);
		return SkeletalMeshComponents;
	}

	return SkeletalMeshComponents;
}

USkeleton* AcquireSkeletonFromObjectGuid(const FGuid& Guid, TSharedPtr<ISequencer> SequencerPtr)
{
	TArray<USkeletalMeshComponent*> SkeletalMeshComponents = AcquireSkeletalMeshComponentsFromObjectGuid(Guid, SequencerPtr);

	if (SkeletalMeshComponents.Num() == 1)
	{
		return GetSkeletonFromComponent(SkeletalMeshComponents[0]);
	}

	return nullptr;
}

FStitchAnimSection::FStitchAnimSection(UMovieSceneSection& InSection)
	: Section(*CastChecked<UMovieSceneStitchAnimSection>(&InSection))
{
}

FStitchAnimSection::~FStitchAnimSection()
{
}

UMovieSceneSection* FStitchAnimSection::GetSectionObject()
{
	return &Section;
}

const FSlateBrush* FStitchAnimSection::GetIconBrush() const
{
	return FMovieScenePoseSearchTracksEditorStyle::Get().GetBrush("Tracks.Stitch");
}

FText FStitchAnimSection::GetSectionTitle() const
{
	if (Section.StitchDatabase != nullptr)
	{
		return FText::Format(LOCTEXT("SectionTitleContentFormat", "{0}"), FText::FromString(Section.StitchDatabase->GetName()));
	}
	return LOCTEXT("NoStitchSection", "No Stitch Database");
}

FText FStitchAnimSection::GetSectionToolTip() const
{
	if (Section.StitchDatabase != nullptr && Section.TargetPoseAsset != nullptr)
	{
		return FText::Format(LOCTEXT("ToolTipContentFormat", "Stitch using database {0} to match pose from {1}"), FText::FromString(Section.StitchDatabase->GetName()), FText::FromString(Section.TargetPoseAsset->GetName()));
	}
	return FText::GetEmpty();
}

float FStitchAnimSection::GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const
{
	return ViewDensity.UniformHeight.Get(StitchAnimEditorConstants::AnimationTrackHeight);
}


FMargin FStitchAnimSection::GetContentPadding() const
{
	return FMargin(8.0f, 8.0f);
}

int32 FStitchAnimSection::OnPaintSection(FSequencerSectionPainter& Painter) const
{
	return Painter.PaintSectionBackground();
}

FStitchAnimTrackEditor::FStitchAnimTrackEditor(TSharedRef<ISequencer> InSequencer)
	: FMovieSceneTrackEditor(InSequencer)
{

}

TSharedRef<ISequencerTrackEditor> FStitchAnimTrackEditor::CreateTrackEditor(TSharedRef<ISequencer> InSequencer)
{
	return MakeShareable(new FStitchAnimTrackEditor(InSequencer));
}

bool FStitchAnimTrackEditor::SupportsSequence(UMovieSceneSequence* InSequence) const
{
	ETrackSupport TrackSupported = InSequence ? InSequence->IsTrackSupported(UMovieSceneStitchAnimTrack::StaticClass()) : ETrackSupport::NotSupported;
	return TrackSupported != ETrackSupport::NotSupported;
}

bool FStitchAnimTrackEditor::SupportsType(TSubclassOf<UMovieSceneTrack> Type) const
{
	return Type == UMovieSceneStitchAnimTrack::StaticClass();
}

TSharedRef<ISequencerSection> FStitchAnimTrackEditor::MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding)
{
	check(SupportsType(SectionObject.GetOuter()->GetClass()));

	return MakeShareable(new FStitchAnimSection(SectionObject));
}

bool FStitchAnimTrackEditor::HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid)
{
	using namespace UE::PoseSearch;
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	if (Asset->IsA<UPoseSearchDatabase>() && SequencerPtr.IsValid())
	{
		UPoseSearchDatabase* PoseSearchDatabase = Cast<UPoseSearchDatabase>(Asset);

		if (TargetObjectGuid.IsValid() && PoseSearchDatabase)
		{
			USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(TargetObjectGuid, GetSequencer());

			// TODO: Check skeleton compatibility? Do I need to specify a role for the database?
			//if (Skeleton && Skeleton->IsCompatibleForEditor(AnimSequence->GetSkeleton()))
			{
				UObject* Object = SequencerPtr->FindSpawnedObjectOrTemplate(TargetObjectGuid);

				UMovieSceneTrack* Track = nullptr;

				const FScopedTransaction Transaction(LOCTEXT("AddStitchedAnim_Transaction", "Add Stitched Animation"));

				int32 RowIndex = INDEX_NONE;
				TFunction<void(UMovieSceneSection*)> SectionCallback = nullptr;
				AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FStitchAnimTrackEditor::AddKeyInternal, Object, PoseSearchDatabase, Track, RowIndex, SectionCallback));

				return true;
			}
		}
	}
	return false;
}


void FStitchAnimTrackEditor::OnInitialize()
{
	// Register as animation mixer item menu provider
	IModularFeatures::Get().RegisterModularFeature(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName(), this);
}

void FStitchAnimTrackEditor::OnRelease()
{
	// Unregister as animation mixer item menu provider
	IModularFeatures::Get().UnregisterModularFeature(IMovieSceneAnimMixerItemMenuProvider::GetModularFeatureName(), this);
}

void FStitchAnimTrackEditor::BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass)
{
	// Stitch Animation menu entries are now added via PopulateObjectBindingAnimationMenu,
	// which is called by FCommonAnimationTrackEditor::BuildObjectBindingTrackMenu
}

void FStitchAnimTrackEditor::PopulateObjectBindingAnimationMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass, bool bIsInsideSubmenu)
{
	using namespace UE::PoseSearch;
	if (ObjectClass->IsChildOf(USkeletalMeshComponent::StaticClass()) || ObjectClass->IsChildOf(AActor::StaticClass()) || ObjectClass->IsChildOf(UChildActorComponent::StaticClass()))
	{
		const TSharedPtr<ISequencer> ParentSequencer = GetSequencer();

		USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(ObjectBindings[0], GetSequencer());

		if (Skeleton)
		{
			// Load the asset registry module
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

			// Collect a full list of assets with the specified class
			TArray<FAssetData> AssetDataList;
			AssetRegistryModule.Get().GetAssetsByClass(UPoseSearchDatabase::StaticClass()->GetClassPathName(), AssetDataList, true);

			if (AssetDataList.Num())
			{
				UMovieSceneTrack* Track = nullptr;
				TFunction<void(UMovieSceneSection*)> SectionCallback = nullptr;

				MenuBuilder.AddSubMenu(
					LOCTEXT("AddStitchAnimation", "Stitch Animation"), NSLOCTEXT("Sequencer", "AddStitchAnimationTooltip", "Adds a stitch animation track."),
					FNewMenuDelegate::CreateRaw(this, &FStitchAnimTrackEditor::AddAnimationSubMenu, ObjectBindings, Skeleton, Track, SectionCallback),
					/*bInOpenSubMenuOnClick=*/ false,
					FSlateIcon(FMovieScenePoseSearchTracksEditorStyle::Get().GetStyleSetName(), "Tracks.Stitch")
				);
			}
		}
	}
}

TSharedRef<SWidget> FStitchAnimTrackEditor::BuildAddAnimationSubMenu(FGuid ObjectBinding, USkeleton* Skeleton, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> WeakViewModel, TFunction<void(UMovieSceneSection*)> InSectionCallback)
{
	using namespace UE::Sequencer;
	TViewModelPtr<FViewModel> ViewModel = WeakViewModel.Pin();
	if (!ViewModel)
	{
		return SNullWidget::NullWidget;
	}

	TViewModelPtr<ITrackExtension> TrackModel = ViewModel->FindAncestorOfType<ITrackExtension>(true);

	if (!TrackModel)
	{
		return SNullWidget::NullWidget;
	}

	UMovieSceneTrack* Track = TrackModel->GetTrack();

	FMenuBuilder MenuBuilder(true, nullptr);

	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBinding);

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("AddStitchAnimation_Label", "Add Stitch Animation"));
	{
		AddAnimationSubMenu(MenuBuilder, ObjectBindings, Skeleton, Track, InSectionCallback);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FStitchAnimTrackEditor::BuildAnimationSubMenu(FGuid ObjectBinding, USkeleton* Skeleton, UMovieSceneTrack* Track, TFunction<void(UMovieSceneSection*)> InSectionCallback)
{
	FMenuBuilder MenuBuilder(true, nullptr);

	TArray<FGuid> ObjectBindings;
	ObjectBindings.Add(ObjectBinding);

	AddAnimationSubMenu(MenuBuilder, ObjectBindings, Skeleton, Track, InSectionCallback);

	return MenuBuilder.MakeWidget();
}

void FStitchAnimTrackEditor::AddAnimationSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, USkeleton* Skeleton, UMovieSceneTrack* Track, TFunction<void(UMovieSceneSection*)> InSectionCallback)
{
	using namespace UE::PoseSearch;
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	UMovieSceneSequence* Sequence = SequencerPtr.IsValid() ? SequencerPtr->GetFocusedMovieSceneSequence() : nullptr;

	FAssetPickerConfig AssetPickerConfig;
	{
		AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateRaw(this, &FStitchAnimTrackEditor::OnAnimationDatabaseAssetSelected, ObjectBindings, Track, InSectionCallback);
		AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateRaw(this, &FStitchAnimTrackEditor::OnAnimationDatabaseAssetEnterPressed, ObjectBindings, Track, InSectionCallback);
		AssetPickerConfig.bAllowNullSelection = false;
		AssetPickerConfig.bAddFilterUI = true;
		AssetPickerConfig.bShowTypeInColumnView = false;
		AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
		AssetPickerConfig.Filter.bRecursiveClasses = true;
		AssetPickerConfig.Filter.ClassPaths.Add(UPoseSearchDatabase::StaticClass()->GetClassPathName());
		AssetPickerConfig.SaveSettingsName = TEXT("SequencerAssetPicker");
		AssetPickerConfig.AdditionalReferencingAssets.Add(FAssetData(Sequence));
	}

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	const float WidthOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserWidth() : 500.f;
	const float HeightOverride = SequencerPtr.IsValid() ? SequencerPtr->GetSequencerSettings()->GetAssetBrowserHeight() : 400.f;

	TSharedPtr<SBox> MenuEntry = SNew(SBox)
		.WidthOverride(WidthOverride)
		.HeightOverride(HeightOverride)
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		];

	MenuBuilder.AddWidget(MenuEntry.ToSharedRef(), FText::GetEmpty(), true);
}

void FStitchAnimTrackEditor::OnAnimationDatabaseAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TFunction<void(UMovieSceneSection*)> InSectionCallback)
{
	using namespace UE::PoseSearch;
	FSlateApplication::Get().DismissAllMenus();

	UObject* SelectedObject = AssetData.GetAsset();
	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();

	if (SelectedObject && SelectedObject->IsA(UPoseSearchDatabase::StaticClass()) && SequencerPtr.IsValid())
	{
		UPoseSearchDatabase* PoseSearchDatabase = CastChecked<UPoseSearchDatabase>(AssetData.GetAsset());

		const FScopedTransaction Transaction(LOCTEXT("AddStitchAnim_Transaction", "Add Stitch Animation"));

		for (FGuid ObjectBinding : ObjectBindings)
		{
			UObject* Object = SequencerPtr->FindSpawnedObjectOrTemplate(ObjectBinding);
			int32 RowIndex = INDEX_NONE;
			AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FStitchAnimTrackEditor::AddKeyInternal, Object, PoseSearchDatabase, Track, RowIndex, InSectionCallback));
		}
	}
}

void FStitchAnimTrackEditor::OnAnimationDatabaseAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TFunction<void(UMovieSceneSection*)> InSectionCallback)
{
	if (AssetData.Num() > 0)
	{
		OnAnimationDatabaseAssetSelected(AssetData[0].GetAsset(), ObjectBindings, Track, InSectionCallback);
	}
}


FKeyPropertyResult FStitchAnimTrackEditor::AddKeyInternal(FFrameNumber KeyTime, UObject* Object, UPoseSearchDatabase* PoseSearchDatabase, UMovieSceneTrack* Track, int32 RowIndex, TFunction<void(UMovieSceneSection*)> InSectionCallback)
{
	FKeyPropertyResult KeyPropertyResult;

	FFindOrCreateHandleResult HandleResult = FindOrCreateHandleToObject(Object);
	FGuid ObjectHandle = HandleResult.Handle;
	KeyPropertyResult.bHandleCreated |= HandleResult.bWasCreated;
	if (ObjectHandle.IsValid())
	{
		UMovieScene* MovieScene = GetSequencer()->GetFocusedMovieSceneSequence()->GetMovieScene();

		// If a callback is provided (mixer case), create the section directly and let the callback handle adding it
		if (InSectionCallback)
		{
			// Create section with the track as outer so MakeSectionInterface can find the mixer track
			UMovieSceneStitchAnimSection* NewSection = NewObject<UMovieSceneStitchAnimSection>(Track, NAME_None, RF_Transactional);
			NewSection->StitchDatabase = PoseSearchDatabase;

			// Set up default section timing (duplicating what AddNewAnimationOnRow does)
			NewSection->SetRange(TRange<FFrameNumber>(KeyTime, KeyTime + (int32)FMath::FloorToInt(3.0f * MovieScene->GetTickResolution().AsDecimal())));

			UE::MovieScene::AddDefaultStitchRootMotionDecoration(NewSection);

			// Call the callback to add the section to the track
			InSectionCallback(NewSection);

			KeyPropertyResult.bTrackModified = true;
			KeyPropertyResult.SectionsCreated.Add(NewSection);

			GetSequencer()->EmptySelection();
			GetSequencer()->SelectSection(NewSection);
			GetSequencer()->ThrobSectionSelection();
		}
		else
		{
			// Standard path: create dedicated track and add section normally
			UMovieSceneStitchAnimTrack* StitchAnimTrack = Cast<UMovieSceneStitchAnimTrack>(Track);
			FMovieSceneBinding* Binding = MovieScene->FindBinding(ObjectHandle);

			// Add a track if no track was specified or if the track specified doesn't belong to the tracks of the targeted guid
			if (!StitchAnimTrack || (Binding && !Binding->GetTracks().Contains(StitchAnimTrack)))
			{
				StitchAnimTrack = CastChecked<UMovieSceneStitchAnimTrack>(AddTrack(MovieScene, ObjectHandle, UMovieSceneStitchAnimTrack::StaticClass(), NAME_None), ECastCheckedType::NullAllowed);
				KeyPropertyResult.bTrackCreated = true;
			}

			if (ensure(StitchAnimTrack))
			{
				StitchAnimTrack->Modify();

				UMovieSceneStitchAnimSection* NewSection = Cast<UMovieSceneStitchAnimSection>(StitchAnimTrack->AddNewAnimationOnRow(KeyTime, PoseSearchDatabase, RowIndex));
				UE::MovieScene::AddDefaultStitchRootMotionDecoration(NewSection);

				KeyPropertyResult.bTrackModified = true;
				KeyPropertyResult.SectionsCreated.Add(NewSection);

				GetSequencer()->EmptySelection();
				GetSequencer()->SelectSection(NewSection);
				GetSequencer()->ThrobSectionSelection();
			}
		}
	}

	return KeyPropertyResult;
}

const UClass* FStitchAnimTrackEditor::GetHandledMixerItemClass() const
{
	return UMovieSceneStitchAnimSection::StaticClass();
}

void FStitchAnimTrackEditor::PopulateAddMixerItemMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TSharedPtr<ISequencer> Sequencer, int32 RowIndex)
{
	using namespace UE::PoseSearch;

	if (ObjectBindings.Num() == 0 || !Sequencer.IsValid() || !Track)
	{
		return;
	}

	USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(ObjectBindings[0], Sequencer);
	if (!Skeleton)
	{
		return;
	}

	// Create a callback that will add the section to the mixer track at the specified row or with auto row
	auto SectionCallback = [Track, RowIndex](UMovieSceneSection* Section)
	{
		if (UMovieSceneCommonAnimationTrack* CommonTrack = Cast<UMovieSceneCommonAnimationTrack>(Track))
		{
			if (RowIndex != INDEX_NONE)
			{
				// Use the specified row index
				Section->SetRowIndex(RowIndex);
				static_cast<UMovieSceneTrack*>(CommonTrack)->AddSection(UMovieSceneTrack::FSectionParameter{*Section});
			}
			else
			{
				// Use auto-row assignment
				CommonTrack->AddSectionWithAutoRow(Section);
			}
		}
	};

	// Wrap the stitch animation menu in a submenu
	MenuBuilder.AddSubMenu(
		LOCTEXT("AddStitchAnimation", "Stitch Animation"),
		LOCTEXT("AddStitchAnimationTooltip", "Adds a stitch animation section"),
		FNewMenuDelegate::CreateLambda([this, ObjectBindings, Skeleton, Track, SectionCallback](FMenuBuilder& SubMenuBuilder)
		{
			AddAnimationSubMenu(SubMenuBuilder, ObjectBindings, Skeleton, Track, SectionCallback);
		}),
		/*bInOpenSubMenuOnClick=*/ false,
		FSlateIcon(FMovieScenePoseSearchTracksEditorStyle::Get().GetStyleSetName(), "Tracks.Stitch")
	);
}

TSharedPtr<SWidget> FStitchAnimTrackEditor::BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params)
{
	USkeleton* Skeleton = AcquireSkeletonFromObjectGuid(ObjectBinding, GetSequencer());

	if (Skeleton)
	{
		TFunction<void(UMovieSceneSection*)> SectionCallback = nullptr;
		FOnGetContent HandleGetAddButtonContent = FOnGetContent::CreateSP(this, &FStitchAnimTrackEditor::BuildAddAnimationSubMenu, ObjectBinding, Skeleton, Params.ViewModel.AsWeak(), SectionCallback);
		return UE::Sequencer::MakeAddButton(LOCTEXT("AnimationText", "Stitch Animation"), HandleGetAddButtonContent, Params.ViewModel);
	}
	else
	{
		return TSharedPtr<SWidget>();
	}
}

bool FStitchAnimTrackEditor::OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams)
{
	using namespace UE::PoseSearch;
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (!Operation.IsValid() || !Operation->IsOfType<FAssetDragDropOp>())
	{
		return false;
	}

	if (!DragDropParams.TargetObjectGuid.IsValid())
	{
		return false;
	}

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return false;
	}

	UMovieSceneSequence* FocusedSequence = SequencerPtr->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return false;
	}

	TArray<USkeletalMeshComponent*> SkeletalMeshComponents = AcquireSkeletalMeshComponentsFromObjectGuid(DragDropParams.TargetObjectGuid, SequencerPtr, false);

	TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);

	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(FocusedSequence, AssetData))
		{
			continue;
		}

		UPoseSearchDatabase* PoseSearchDatabase = Cast<UPoseSearchDatabase>(AssetData.GetAsset());

		for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
		{
			USkeleton* Skeleton = GetSkeletonFromComponent(SkeletalMeshComponent);
			if (PoseSearchDatabase != nullptr)
			//if (bValidAnimSequence && Skeleton && Skeleton->IsCompatibleForEditor(AnimSequence->GetSkeleton()))
			{
				FFrameRate TickResolution = SequencerPtr->GetFocusedTickResolution();
				FFrameNumber LengthInFrames = TickResolution.AsFrameNumber(3.0f);
				DragDropParams.FrameRange = TRange<FFrameNumber>(DragDropParams.FrameNumber, DragDropParams.FrameNumber + LengthInFrames);
				return true;
			}
		}
	}

	return false;
}


FReply FStitchAnimTrackEditor::OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams)
{
	using namespace UE::PoseSearch;
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (!Operation.IsValid() || !Operation->IsOfType<FAssetDragDropOp>())
	{
		return FReply::Unhandled();
	}

	if (!DragDropParams.TargetObjectGuid.IsValid())
	{
		return FReply::Unhandled();
	}

	TSharedPtr<ISequencer> SequencerPtr = GetSequencer();
	if (!SequencerPtr)
	{
		return FReply::Unhandled();
	}

	UMovieSceneSequence* FocusedSequence = SequencerPtr->GetFocusedMovieSceneSequence();
	if (!FocusedSequence)
	{
		return FReply::Unhandled();
	}

	TArray<USkeletalMeshComponent*> SkeletalMeshComponents = AcquireSkeletalMeshComponentsFromObjectGuid(DragDropParams.TargetObjectGuid, SequencerPtr, false);

	const FScopedTransaction Transaction(LOCTEXT("DropAssets", "Drop Assets"));

	TSharedPtr<FAssetDragDropOp> DragDropOp = StaticCastSharedPtr<FAssetDragDropOp>(Operation);

	FMovieSceneTrackEditor::BeginKeying(DragDropParams.FrameNumber);

	bool bAnyDropped = false;
	for (const FAssetData& AssetData : DragDropOp->GetAssets())
	{
		if (!MovieSceneToolHelpers::IsValidAsset(FocusedSequence, AssetData))
		{
			continue;
		}

		UPoseSearchDatabase* PoseSearchDatabase = Cast<UPoseSearchDatabase>(AssetData.GetAsset());
		for (USkeletalMeshComponent* SkeletalMeshComponent : SkeletalMeshComponents)
		{
			USkeleton* Skeleton = GetSkeletonFromComponent(SkeletalMeshComponent);

			if (PoseSearchDatabase != nullptr)// && Skeleton && Skeleton->IsCompatibleForEditor(AnimSequence->GetSkeleton()))
			{
				UObject* BoundObject = SequencerPtr.IsValid() ? SequencerPtr->FindSpawnedObjectOrTemplate(DragDropParams.TargetObjectGuid) : nullptr;

				TFunction<void(UMovieSceneSection*)> SectionCallback = nullptr;
				AnimatablePropertyChanged(FOnKeyProperty::CreateRaw(this, &FStitchAnimTrackEditor::AddKeyInternal, BoundObject, PoseSearchDatabase, DragDropParams.Track.Get(), DragDropParams.RowIndex, SectionCallback));

				bAnyDropped = true;
			}
		}
	}

	FMovieSceneTrackEditor::EndKeying();

	return bAnyDropped ? FReply::Handled() : FReply::Unhandled();
}


#undef LOCTEXT_NAMESPACE
