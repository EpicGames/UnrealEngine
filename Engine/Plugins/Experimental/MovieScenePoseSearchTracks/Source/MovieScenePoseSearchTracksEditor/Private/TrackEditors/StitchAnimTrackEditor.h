// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/Guid.h"
#include "Templates/SubclassOf.h"
#include "Templates/UniquePtr.h"
#include "Widgets/SWidget.h"
#include "ISequencer.h"
#include "MovieSceneTrack.h"
#include "ISequencerSection.h"
#include "ISequencerTrackEditor.h"
#include "MovieSceneTrackEditor.h"
#include "SequencerCoreFwd.h"
#include "IMovieSceneAnimMixerItemMenuProvider.h"

namespace UE::Sequencer
{
	class ITrackExtension;
}

struct FAssetData;
class FMenuBuilder;
class FSequencerSectionPainter;
class UMovieSceneStitchAnimSection;
class USkeleton;
class UPoseSearchDatabase;

/**
 * Tools for stitch anim tracks
 */
class FStitchAnimTrackEditor : public FMovieSceneTrackEditor
							 , public IMovieSceneAnimMixerItemMenuProvider
{
public:
	/**
	 * Constructor
	 *
	 * @param InSequencer The sequencer instance to be used by this tool
	 */
	FStitchAnimTrackEditor(TSharedRef<ISequencer> InSequencer);

	/** Virtual destructor. */
	virtual ~FStitchAnimTrackEditor() { }

	/**
	 * Creates an instance of this class.  Called by a sequencer
	 *
	 * @param OwningSequencer The sequencer instance to be used by this tool
	 * @return The new instance of this class
	 */
	static TSharedRef<ISequencerTrackEditor> CreateTrackEditor(TSharedRef<ISequencer> OwningSequencer);

public:

	// ISequencerTrackEditor interface
	virtual void OnInitialize() override;
	virtual void OnRelease() override;
	virtual void BuildObjectBindingTrackMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass) override;
	virtual bool HandleAssetAdded(UObject* Asset, const FGuid& TargetObjectGuid) override;
	virtual TSharedRef<ISequencerSection> MakeSectionInterface(UMovieSceneSection& SectionObject, UMovieSceneTrack& Track, FGuid ObjectBinding) override;
	virtual bool SupportsSequence(UMovieSceneSequence* InSequence) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneTrack> Type) const override;
	virtual TSharedPtr<SWidget> BuildOutlinerEditWidget(const FGuid& ObjectBinding, UMovieSceneTrack* Track, const FBuildEditWidgetParams& Params) override;
	virtual bool OnAllowDrop(const FDragDropEvent& DragDropEvent, FSequencerDragDropParams& DragDropParams) override;
	virtual FReply OnDrop(const FDragDropEvent& DragDropEvent, const FSequencerDragDropParams& DragDropParams) override;

private:

	/** Animation sub menu */
	TSharedRef<SWidget> BuildAddAnimationSubMenu(FGuid ObjectBinding, USkeleton* Skeleton, UE::Sequencer::TWeakViewModelPtr<UE::Sequencer::FViewModel> WeakViewModel, TFunction<void(UMovieSceneSection*)> InSectionCallback = nullptr);
	TSharedRef<SWidget> BuildAnimationSubMenu(FGuid ObjectBinding, USkeleton* Skeleton, UMovieSceneTrack* Track, TFunction<void(UMovieSceneSection*)> InSectionCallback = nullptr);
	void AddAnimationSubMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, USkeleton* Skeleton, UMovieSceneTrack* Track, TFunction<void(UMovieSceneSection*)> InSectionCallback = nullptr);

	/** Animation asset selected */
	void OnAnimationDatabaseAssetSelected(const FAssetData& AssetData, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TFunction<void(UMovieSceneSection*)> InSectionCallback);

	/** Animation asset enter pressed */
	void OnAnimationDatabaseAssetEnterPressed(const TArray<FAssetData>& AssetData, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TFunction<void(UMovieSceneSection*)> InSectionCallback);

	/** Delegate for AnimatablePropertyChanged in AddKey */
	FKeyPropertyResult AddKeyInternal(FFrameNumber KeyTime, UObject* Object, UPoseSearchDatabase* PoseSearchDatabase, UMovieSceneTrack* Track, int32 RowIndex, TFunction<void(UMovieSceneSection*)> InSectionCallback);

	// IMovieSceneAnimMixerItemMenuProvider interface
	const UClass* GetHandledMixerItemClass() const override;
	void PopulateAddMixerItemMenu(FMenuBuilder& MenuBuilder, TArray<FGuid> ObjectBindings, UMovieSceneTrack* Track, TSharedPtr<ISequencer> Sequencer, int32 RowIndex) override;
	void PopulateObjectBindingAnimationMenu(FMenuBuilder& MenuBuilder, const TArray<FGuid>& ObjectBindings, const UClass* ObjectClass, bool bIsInsideSubmenu) override;
};


/** Class for stitch anim sections */
class FStitchAnimSection
	: public ISequencerSection
	, public TSharedFromThis<FStitchAnimSection>
{
public:

	/** Constructor. */
	FStitchAnimSection(UMovieSceneSection& InSection);

	/** Virtual destructor. */
	virtual ~FStitchAnimSection();

public:

	// ISequencerSection interface

	virtual UMovieSceneSection* GetSectionObject() override;
	virtual const FSlateBrush* GetIconBrush() const override;
	virtual FText GetSectionTitle() const override;
	virtual FText GetSectionToolTip() const override;
	virtual float GetSectionHeight(const UE::Sequencer::FViewDensityInfo& ViewDensity) const override;
	virtual FMargin GetContentPadding() const override;
	virtual int32 OnPaintSection(FSequencerSectionPainter& Painter) const override;

private:

	/** The section we are visualizing */
	UMovieSceneStitchAnimSection& Section;

	TUniquePtr<FMovieSceneSequenceTransform> InitialDragTransform;
};
